{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Remote Control Interception API",
	"Description": "Plugin that allows to intercept Remote Control commands",
	"Category": "Messaging",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"Installed": false,
	"Hidden": true,
	"Modules": [
		{
			"Name": "RemoteControlInterception",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}
