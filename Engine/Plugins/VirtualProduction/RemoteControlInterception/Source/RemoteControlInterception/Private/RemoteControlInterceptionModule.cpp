// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlInterceptionModule.h"

#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FRemoteControlInterceptionModule, RemoteControlInterception)
