// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"


/**
 * RemoteControlInterception module implementation
 */
class FRemoteControlInterceptionModule :
	public IModuleInterface
{
public:
	FRemoteControlInterceptionModule() = default;
	virtual ~FRemoteControlInterceptionModule() = default;
};
