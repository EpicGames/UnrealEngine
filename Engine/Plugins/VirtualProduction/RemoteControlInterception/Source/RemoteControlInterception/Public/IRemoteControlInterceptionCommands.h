// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FRCIObjectMetadata;
struct FRCIFunctionMetadata;
struct FRCIPropertiesMetadata;
struct FRCIControllerMetadata;

/**
 * Interception flags that define how RemoteControl should behave after a message was intercepted.
 */
enum class ERCIResponse : uint8
{
	// Set/reset property on RemoteControl side even though the message was intercepted
	Apply,
	// Do not process the RC command on RemoteControl side. An interceptor will decide what to do with it.
	Intercept,
};

/**
 * Payload serialization type (proxy type for ERCPayloadType to avoid RC module dependency)
 */
enum class ERCIPayloadType : uint8
{
	Cbor,
	Json,
};

/**
 * Remote property access mode  (proxy type for ERCAccess to avoid RC module dependency)
 */
enum class ERCIAccess : uint8
{
	NO_ACCESS,
	READ_ACCESS,
	WRITE_ACCESS,
	WRITE_TRANSACTION_ACCESS,
	WRITE_MANUAL_TRANSACTION_ACCESS,
};

/**
 * Type of operation to perform when setting a remote property's value (proxy type for ERCModifyOperation to avoid module dependency)
 */
enum class ERCIModifyOperation : uint8
{
	EQUAL,
	ADD,
	SUBTRACT,
	MULTIPLY,
	DIVIDE
};


/**
 * The list of remote control commands available for interception
 */
template <class TResponseType>
class IRemoteControlInterceptionCommands
{
public:
	virtual ~IRemoteControlInterceptionCommands() = default;

public:
	/**
	 * SetObjectProperty command to process
	 *
	 * @param InObjectProperties - Metadata of the object and its properties that are going to be modified
	 *
	 * @return - Return types depends on the interface implementation (template parameter)
	 */
	virtual TResponseType SetObjectProperties(FRCIPropertiesMetadata& InObjectProperties) = 0;

	/**
	 * ResetObject command to process
	 *
	 * @param InObject - Metadata of the object and its properties that are going to be modified
	 *
	 * @return - Return types depends on the interface implementation (template parameter)
	 * @param OutResponse - interception response flag (not required for the handlers)
	 */
	virtual TResponseType ResetObjectProperties(FRCIObjectMetadata& InObject) = 0;
	
	/**
	 * InvokeCall command to process
	 *
	 * @param InFunction - Metadata of the UFunction that are going to be modified
	 *
	 * @return - Return types depends on the interface implementation (template parameter)
	 */
	virtual TResponseType InvokeCall(FRCIFunctionMetadata& InFunction) = 0;

	/**
	 * SetPresetController command
	 *
	 * @param InController - Metadata of the controller being manipulated
	 *
	 * @return - Return types depends on the interface implementation (template parameter)
	 */
	virtual TResponseType SetPresetController(FRCIControllerMetadata& InController) = 0;
};


/**
 * Object metadata for custom interception/replication/processing purposes
 */
struct FRCIObjectMetadata
{
public:
	FRCIObjectMetadata()
	{ }

	FRCIObjectMetadata(const FString& InObjectPath, const FString& InPropertyPath, const FString& InPropertyPathInfo, const ERCIAccess InAccess)
		: ObjectPath(InObjectPath)
		, PropertyPath(InPropertyPath)
		, PropertyPathInfo(InPropertyPathInfo)
		, Access(InAccess)
	{
		UniquePath = *FString::Printf(TEXT("%s:%s"), *ObjectPath, *PropertyPath);
	}

	virtual ~FRCIObjectMetadata() = default;

public:
	/** Get Unique Path for Object metadata */
	const FName& GetUniquePath() const
	{ return UniquePath; }
	
	/** Returns true if reference is valid */
	virtual bool IsValid() const
	{
		return !ObjectPath.IsEmpty() && !PropertyPath.IsEmpty() && !PropertyPathInfo.IsEmpty();
	}

	/**
	 * FRCObjectMetadata serialization
	 *
	 * @param Ar       - The archive
	 * @param Interception - Instance to serialize/deserialize
	 *
	 * @return FArchive instance.
	 */
	friend FArchive& operator<<(FArchive& Ar, FRCIObjectMetadata& Interception)
	{
		Ar << Interception.ObjectPath;
		Ar << Interception.PropertyPath;
		Ar << Interception.PropertyPathInfo;
		Ar << Interception.Access;
		return Ar;
	}

public:
	/** Owner object path */
	FString ObjectPath;
	
	/** FProperty path */
	FString PropertyPath;

	/** Full property path, including path to inner structs, arrays, maps, etc */
	FString PropertyPathInfo;

	/** Property access type */
	ERCIAccess Access = ERCIAccess::NO_ACCESS;

public:
	/** Structure Name */
	static constexpr TCHAR const* Name = TEXT("RCIObjectMetadata");
	
private:
	/** Object Path + Property Path */
    FName UniquePath;
};


/**
 * Object properties metadata for custom interception/replication/processing purposes
 */
struct FRCIPropertiesMetadata : public FRCIObjectMetadata
{
public:
	FRCIPropertiesMetadata() = default;

	FRCIPropertiesMetadata(const FString& InObjectPath, const FString& InPropertyPath, const FString& InPropertyPathInfo, const ERCIAccess InAccess, const ERCIPayloadType InPayloadType, const ERCIModifyOperation InOperation, const TArray<uint8>& InPayload)
		: FRCIObjectMetadata(InObjectPath, InPropertyPath, InPropertyPathInfo, InAccess)
		, PayloadType(InPayloadType)
		, Operation(InOperation)
		, Payload(InPayload)
	{ }

	virtual ~FRCIPropertiesMetadata() = default;

public:
	//~ Begin FRCObjectMetadata
	virtual bool IsValid() const override
	{
		return FRCIObjectMetadata::IsValid() && Payload.Num() > 0;
	}
	//~ End FRCObjectMetadata

	/**
	 * FRCObjectPropertiesMetadata serialization
	 *
	 * @param Ar       - The archive
	 * @param Instance - Instance to serialize/deserialize
	 *
	 * @return FArchive instance.
	 */
	friend FArchive& operator<<(FArchive& Ar, FRCIPropertiesMetadata& Instance)
	{
		// Call archive from the parent class
		FRCIObjectMetadata& Super = static_cast<FRCIObjectMetadata&>(Instance);
		Ar << Super;
		
		Ar << Instance.PayloadType;
		Ar << Instance.Operation;
		// const cast is needed for keeping the const& on a constructor and as a class parameter
		Ar << const_cast<TArray<uint8>&>(Instance.Payload);
		return Ar;
	}

public:
	/** Intercepted payload type */
	ERCIPayloadType PayloadType = ERCIPayloadType::Json;

	/** Type of operation to perform on the property **/
	ERCIModifyOperation Operation;

	/** Property payload to intercept */
	const TArray<uint8> Payload;

public:
	/** Structure Name */
	static constexpr TCHAR const* Name = TEXT("RCIPropertiesMetadata");
};

/**
* UFunction metadata for custom interception/replication/processing purposes
*/
struct FRCIFunctionMetadata
{
public:
	FRCIFunctionMetadata()
		: bGenerateTransaction(false)
		, PayloadType(ERCIPayloadType::Json)
	{ }

	FRCIFunctionMetadata(const FString& InObjectPath, const FString& InFunctionPath, const bool bInGenerateTransaction, const ERCIPayloadType InPayloadType, const TArray<uint8>& InPayload)
		: ObjectPath(InObjectPath)
		, FunctionPath(InFunctionPath)
		, bGenerateTransaction(bInGenerateTransaction)
		, PayloadType(InPayloadType)
		, Payload(InPayload)
	{
		UniquePath = *FString::Printf(TEXT("%s:%s"), *ObjectPath, *FunctionPath);
	}

public:
	/** Get Unique Path for Function metadata */
	const FName& GetUniquePath() const
	{ return UniquePath; }

	/** Returns true if reference is valid */
	bool IsValid() const
	{
		return !ObjectPath.IsEmpty() && !FunctionPath.IsEmpty() && Payload.Num() > 0;
	}

	/**
	* FRCObjectMetadata serialization
	*
	* @param Ar				- The archive
	* @param Interception	- Instance to serialize/deserialize
	*
	* @return FArchive instance.
	*/
	friend FArchive& operator<<(FArchive& Ar, FRCIFunctionMetadata& Interception)
	{
		Ar << Interception.ObjectPath;
		Ar << Interception.FunctionPath;
		Ar << Interception.bGenerateTransaction;
		Ar << Interception.PayloadType;
		// const cast is needed for keeping the const& on a constructor and as a class parameter
		Ar << const_cast<TArray<uint8>&>(Interception.Payload);
		return Ar;
	}

public:
	/** Owner object path */
	FString ObjectPath;

	/** UFunction object path */
	FString FunctionPath;

	/** Should the call generate transaction */
	bool bGenerateTransaction;

	/** Intercepted payload type */
	ERCIPayloadType PayloadType;

	/** Property payload to intercept */
	const TArray<uint8> Payload;

public:
	/** Structure Name */
	static constexpr TCHAR const* Name = TEXT("RCIFunctionMetadata");

private:
	/** Object Path + Function Path */
	FName UniquePath;
};

/**
 * Controller properties for interception of Controller get/set operations
 */
struct FRCIControllerMetadata
{
public:

	FRCIControllerMetadata() = default;

	FRCIControllerMetadata(const FName InPreset, const FName InController, const TArray<uint8>& InPayload)
		: Preset(InPreset)
		, Controller(InController)
		, Payload(InPayload)
	{
		UniquePath = *FString::Printf(TEXT("%s:%s"), *Preset.ToString(), *Controller.ToString());
	}

public:
	/** Get Unique Path for Controller metadata */
	const FName& GetUniquePath() const
	{
		return UniquePath;
	}

	/** Returns true if reference is valid */
	bool IsValid() const
	{
		return !Preset.IsNone() && !Controller.IsNone() && Payload.Num() > 0;
	}

	/**
	* FRCIControllerMetadata serialization
	*
	* @param Ar				- The archive
	* @param Interception	- Instance to serialize/deserialize
	*
	* @return FArchive instance.
	*/
	friend FArchive& operator<<(FArchive& Ar, FRCIControllerMetadata& Interception)
	{
		Ar << Interception.Preset;
		Ar << Interception.Controller;
		// const cast is needed for keeping the const& on a constructor and as a class parameter
		Ar << const_cast<TArray<uint8>&>(Interception.Payload);
		return Ar;
	}

public:
	/**Name of the Remote Control Preset holding the Controller */
	FName Preset;

	/** The Controller being manipulated in the Preset */
	FName Controller;

	/** Request Payload */
	const TArray<uint8> Payload;

public:
	/** Structure Name - Used by replication event handlers to deserialize event data into
	* the correct structure for a given event*/
	static constexpr TCHAR const* Name = TEXT("FRCIControllerMetadata");

private:
	/** Preset Name + Controller Name*/
	FName UniquePath;
};