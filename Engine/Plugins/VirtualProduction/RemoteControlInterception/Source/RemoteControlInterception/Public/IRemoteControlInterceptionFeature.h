// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Features/IModularFeature.h"
#include "IRemoteControlInterceptionCommands.h"


/**
 * Base RCI feature interface
 */
template <class TResponseType>
class IRemoteControlInterceptionFeature
	: public IModularFeature
	, public IRemoteControlInterceptionCommands<TResponseType>
{
	static_assert(std::is_same<ERCIResponse, TResponseType>::value || std::is_void<TResponseType>::value, "Only \"ERCIResponse\" and \"void\" template parameters are allowed");

public:
	virtual ~IRemoteControlInterceptionFeature() = default;
};


/**
 * RCI command interceptor feature
 */
class IRemoteControlInterceptionFeatureInterceptor
	: public IRemoteControlInterceptionFeature<ERCIResponse>
{
public:
	static FName GetName()
	{
		static const FName FeatureName = TEXT("RemoteControlInterception_Feature_Interceptor");
		return FeatureName;
	}
};


/**
 * RCI command processor feature
 */
class IRemoteControlInterceptionFeatureProcessor
	: public IRemoteControlInterceptionFeature<void>
{
public:
	static FName GetName()
	{
		static const FName FeatureName = TEXT("RemoteControlInterception_Feature_Processor");
		return FeatureName;
	}
};
